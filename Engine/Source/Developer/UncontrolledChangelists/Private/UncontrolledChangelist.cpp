// Copyright Epic Games, Inc. All Rights Reserved.

#include "UncontrolledChangelist.h"

#include "Dom/JsonObject.h"
#include "ISourceControlModule.h"
#include "Misc/Guid.h"

#define LOCTEXT_NAMESPACE "UncontrolledChangelists"

const FGuid FUncontrolledChangelist::DEFAULT_UNCONTROLLED_CHANGELIST_GUID = FGuid(0, 0, 0, 0);

FUncontrolledChangelist::FUncontrolledChangelist()
	: Guid(FGuid::NewGuid())
{
}

FUncontrolledChangelist::FUncontrolledChangelist(const FGuid& InGuid)
	: Guid(InGuid)
{
}

void FUncontrolledChangelist::Serialize(TSharedRef<FJsonObject> OutJsonObject) const
{
	OutJsonObject->SetStringField(GUID_NAME, Guid.ToString());
}

bool FUncontrolledChangelist::Deserialize(const TSharedRef<FJsonObject> InJsonValue)
{
	FString TempStr;

	if (!InJsonValue->TryGetStringField(GUID_NAME, TempStr))
	{
		UE_LOGF(LogSourceControl, Error, "Cannot get field %ls.", GUID_NAME);
		return false;
	}

	if (!FGuid::Parse(TempStr, Guid))
	{
		UE_LOGF(LogSourceControl, Error, "Cannot parse Guid %ls.", *TempStr);
		return false;
	}

	return true;
}

#undef LOCTEXT_NAMESPACE
