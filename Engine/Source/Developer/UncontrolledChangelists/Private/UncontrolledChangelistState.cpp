// Copyright Epic Games, Inc. All Rights Reserved.

#include "UncontrolledChangelistState.h"

#include "Algo/AnyOf.h"
#include "Algo/Copy.h"
#include "Algo/Transform.h"
#include "Dom/JsonObject.h"
#include "Dom/JsonValue.h"
#include "HAL/FileManager.h"
#include "ISourceControlModule.h"
#include "ISourceControlProvider.h"
#include "SourceControlHelpers.h"
#include "SourceControlOperations.h"
#include "UncontrolledChangelist.h"

#define LOCTEXT_NAMESPACE "UncontrolledChangelists"

const FText FUncontrolledChangelistState::DEFAULT_UNCONTROLLED_CHANGELIST_DESCRIPTION = LOCTEXT("DefaultUncontrolledChangelist", "Default Uncontrolled Changelist");

FUncontrolledChangelistState::FUncontrolledChangelistState(const FUncontrolledChangelist& InUncontrolledChangelist)
	: Changelist(InUncontrolledChangelist)
{
}

FUncontrolledChangelistState::FUncontrolledChangelistState(const FUncontrolledChangelist& InUncontrolledChangelist, const FText& InDescription)
	: Changelist(InUncontrolledChangelist)
{
	SetDescription(InDescription);
}

FName FUncontrolledChangelistState::GetIconName() const
{
	return FName("SourceControl.UncontrolledChangelist");
}

FName FUncontrolledChangelistState::GetSmallIconName() const
{
	return FName("SourceControl.UncontrolledChangelist_Small");
}

const FText& FUncontrolledChangelistState::GetDisplayText() const
{
	return Description;
}

const FText& FUncontrolledChangelistState::GetDescriptionText() const
{
	return Description;
}

FText FUncontrolledChangelistState::GetDisplayTooltip() const
{
	return LOCTEXT("UncontrolledStateTooltip", "Uncontrolled: Locally modified outside of revision control");
}

const FDateTime& FUncontrolledChangelistState::GetTimeStamp() const
{
	return TimeStamp;
}

const TSet<FSourceControlStateRef>& FUncontrolledChangelistState::GetFilesStates() const
{
	return Files;
}

const TSet<FString>& FUncontrolledChangelistState::GetOfflineFiles() const
{
	return OfflineFiles;
}

const TSet<FString>& FUncontrolledChangelistState::GetDeletedOfflineFiles() const
{
	return DeletedOfflineFiles;
}

int32 FUncontrolledChangelistState::GetFileCount() const
{
	// Avoid counting DeletedOfflineFiles.
	return Files.Num() + OfflineFiles.Num();
}

TArray<FString> FUncontrolledChangelistState::GetFilenames() const
{
	TArray<FString> Filenames;
	Filenames.Reserve(GetFileCount());

	Algo::Transform(GetFilesStates(), Filenames, [](const TSharedRef<ISourceControlState>& FileState) { return FileState->GetFilename(); });
	Algo::Transform(GetOfflineFiles(), Filenames, [](const FString& Pathname) { return Pathname; });

	return Filenames;
}

bool FUncontrolledChangelistState::ContainsFilename(const FString& PackageFilename) const
{
	if (OfflineFiles.Contains(PackageFilename))
	{
		return true;
	}

	ISourceControlProvider& SourceControlProvider = ISourceControlModule::Get().GetProvider();

	if (FSourceControlStatePtr FileState = SourceControlProvider.GetState(PackageFilename, EStateCacheUsage::Use))
	{
		return Files.Contains(FileState.ToSharedRef());
	}

	return  false;
}

void FUncontrolledChangelistState::Serialize(TSharedRef<FJsonObject> OutJsonObject, const TFunction<bool(const FString&)>& FilenameFilter) const
{
	TArray<TSharedPtr<FJsonValue>> FileValues;

	OutJsonObject->SetStringField(DESCRIPTION_NAME, Description.ToString());

	auto ApplyFilenameFilter = [&FilenameFilter](const FString& Filename)
	{
		return !FilenameFilter || FilenameFilter(Filename);
	};

	Algo::TransformIf(Files, FileValues, [&ApplyFilenameFilter](const FSourceControlStateRef& File) { return ApplyFilenameFilter(File->GetFilename()); }, [](const FSourceControlStateRef& File) { return MakeShareable(new FJsonValueString(File->GetFilename())); });
	Algo::TransformIf(OfflineFiles, FileValues, ApplyFilenameFilter, [](const FString& OfflineFile) { return MakeShareable(new FJsonValueString(OfflineFile)); });
	Algo::TransformIf(DeletedOfflineFiles, FileValues, ApplyFilenameFilter, [](const FString& DeletedOfflineFile) { return MakeShareable(new FJsonValueString(DeletedOfflineFile)); });

	OutJsonObject->SetArrayField(FILES_NAME, MoveTemp(FileValues));
}

bool FUncontrolledChangelistState::Deserialize(const TSharedRef<FJsonObject> InJsonValue)
{
	const TArray<TSharedPtr<FJsonValue>>* FileValues = nullptr;
	FString TempString;

	if (!InJsonValue->TryGetStringField(DESCRIPTION_NAME, TempString) && !InJsonValue->TryGetStringField(NAME_NAME, TempString))
	{
		UE_LOGF(LogSourceControl, Error, "Cannot get field %ls or %ls.", DESCRIPTION_NAME, NAME_NAME);
		return false;
	}

	SetDescription(FText::FromString(TempString));

	if ((!InJsonValue->TryGetArrayField(FILES_NAME, FileValues)) || (FileValues == nullptr))
	{
		UE_LOGF(LogSourceControl, Error, "Cannot get field %ls.", FILES_NAME);
		return false;
	}

	TArray<FString> Filenames;

	Algo::Transform(*FileValues, Filenames, [](const TSharedPtr<FJsonValue>& File)
	{
		return File->AsString();
	});

	AddFiles(Filenames, ECheckFlags::Modified | ECheckFlags::NotCheckedOut);

	return true;
}

bool FUncontrolledChangelistState::AddFiles(const TArray<FString>& InFilenames, const ECheckFlags InCheckFlags)
{
	TArray<FSourceControlStateRef> FileStates;
	ISourceControlProvider& SourceControlProvider = ISourceControlModule::Get().GetProvider();
	IFileManager& FileManager = IFileManager::Get();
	bool bCheckStatus = (InCheckFlags & ECheckFlags::Modified) != ECheckFlags::None;
	bool bCheckCheckout = (InCheckFlags & ECheckFlags::NotCheckedOut) != ECheckFlags::None;
	bool bOutChanged = false;

	if (InFilenames.IsEmpty())
	{
		return bOutChanged;
	}

	// No source control is available, add files to the offline file set.
	if (!SourceControlProvider.IsAvailable())
	{
		int32 OldSize = OfflineFiles.Num();
		int32 OldDeletedSize = DeletedOfflineFiles.Num();

		for (const FString& Filename : SourceControlHelpers::AbsoluteFilenames(InFilenames))
		{
			if (FileManager.FileExists(*Filename))
			{
				OfflineFiles.Add(Filename);
			}
			else
			{
				// Keep in case we source control provider and can determine this file is source controlled
				DeletedOfflineFiles.Add(Filename);
			}
		}
				
		return (OldSize != OfflineFiles.Num()) || (OldDeletedSize != DeletedOfflineFiles.Num());
	}

	if (bCheckStatus)
	{
		auto UpdateStatusOperation = ISourceControlOperation::Create<FUpdateStatus>();
		UpdateStatusOperation->SetUpdateModifiedState(true);
		UpdateStatusOperation->SetUpdateModifiedStateToLocalRevision(true);
		UpdateStatusOperation->SetQuiet(true);
		UpdateStatusOperation->SetForceUpdate(true);
		SourceControlProvider.Execute(UpdateStatusOperation, InFilenames);
	}

	const bool GetStateSucceeded = SourceControlProvider.GetState(InFilenames, FileStates, EStateCacheUsage::Use) == ECommandResult::Succeeded;

	if (GetStateSucceeded && (!FileStates.IsEmpty()))
	{
		for (FSourceControlStateRef FileState : FileStates)
		{
			const bool bIsSourceControlled = (!FileState->IsUnknown()) && FileState->IsSourceControlled();
			const bool bFileExists = FileManager.FileExists(*FileState->GetFilename());

			const bool bIsUncontrolled = (!bIsSourceControlled) && bFileExists;
			// File doesn't exist and is not marked for delete
			const bool bIsDeleted = bIsSourceControlled && (!bFileExists) && (!FileState->IsDeleted());
			const bool bIsModified = FileState->IsModified() && (!FileState->IsDeleted());

			const bool bIsCheckoutCompliant = (!bCheckCheckout) || (!FileState->IsCheckedOut());
			const bool bIsStatusCompliant = (!bCheckStatus) || bIsModified || bIsUncontrolled || bIsDeleted;

			if (bIsCheckoutCompliant && bIsStatusCompliant)
			{
				Files.Add(FileState);
				bOutChanged = true;
			}
		}
	}

	return bOutChanged;
}

bool FUncontrolledChangelistState::RemoveFiles(const TArray<FSourceControlStateRef>& InFileStates)
{
	bool bOutChanged = false;

	for (const FSourceControlStateRef& FileState : InFileStates)
	{
		bOutChanged |= (Files.Remove(FileState) > 0);
	}

	return bOutChanged;
}

bool FUncontrolledChangelistState::RemoveFiles(const TArray<FString>& InFilenames)
{
	bool bOutChanged = false;

	ISourceControlProvider& SourceControlProvider = ISourceControlModule::Get().GetProvider();

	for (const FString& Filename : SourceControlHelpers::AbsoluteFilenames(InFilenames))
	{
		bOutChanged |= (OfflineFiles.Remove(Filename) > 0);
		
		if (FSourceControlStatePtr FileState = SourceControlProvider.GetState(Filename, EStateCacheUsage::Use))
		{
			bOutChanged |= (Files.Remove(FileState.ToSharedRef()) > 0);
		}
	}

	return bOutChanged;
}

bool FUncontrolledChangelistState::UpdateStatus()
{
	TArray<FString> FilesToUpdate;
	bool bOutChanged = false;
	const int32 InitialFileNumber = Files.Num();
	const int32 InitialOfflineFileNumber = OfflineFiles.Num();
	const int32 InitialDeletedOfflineFiles = DeletedOfflineFiles.Num();

	Algo::Transform(Files, FilesToUpdate, [](const FSourceControlStateRef& State) { return State->GetFilename(); });
	Algo::Copy(OfflineFiles, FilesToUpdate);
	Algo::Copy(DeletedOfflineFiles, FilesToUpdate);

	Files.Empty();
	OfflineFiles.Empty();
	DeletedOfflineFiles.Empty();

	if (FilesToUpdate.Num() == 0)
	{
		return bOutChanged;
	}

	bOutChanged |= AddFiles(FilesToUpdate, ECheckFlags::All);

	const bool bFileNumberChanged = InitialFileNumber == Files.Num();
	const bool bOfflineFileNumberChanged = InitialOfflineFileNumber == OfflineFiles.Num();
	const bool bDeletedOfflineFileNumberChanged = InitialDeletedOfflineFiles == DeletedOfflineFiles.Num();

	bOutChanged |= bFileNumberChanged || bOfflineFileNumberChanged || bDeletedOfflineFileNumberChanged;

	return bOutChanged;
}

void FUncontrolledChangelistState::RemoveDuplicates(TSet<FString>& InOutAddedAssets)
{
	for (const FSourceControlStateRef& FileState : Files)
	{
		const FString& Filename = FileState->GetFilename();
		
		InOutAddedAssets.Remove(Filename);
	}

	for (const FString& OfflineFile : OfflineFiles)
	{
		InOutAddedAssets.Remove(OfflineFile);
	}

	for (const FString& DeletedOfflineFile : DeletedOfflineFiles)
	{
		InOutAddedAssets.Remove(DeletedOfflineFile);
	}
}

void FUncontrolledChangelistState::SetDescription(const FText& InDescription)
{
	Description = InDescription;

	if (Description.EqualToCaseIgnored(FText::FromString(TEXT("default"))))
	{
		Description = FText::Format(LOCTEXT("Default_Override", "{0} (Uncontrolled Changelist)"), Description);
	}
}


bool FUncontrolledChangelistState::ContainsFiles() const
{
	// Ignore DeletedOfflineFiles
	return !Files.IsEmpty() || !OfflineFiles.IsEmpty();
}

#undef LOCTEXT_NAMESPACE
