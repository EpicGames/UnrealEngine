// Copyright Epic Games, Inc. All Rights Reserved.

#include "UncontrolledChangelistsModule.h"

#include "CoreGlobals.h"
#include "Algo/AnyOf.h"
#include "Algo/Copy.h"
#include "Algo/Find.h"
#include "Algo/ForEach.h"
#include "Algo/Transform.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Dom/JsonObject.h"
#include "FileHelpers.h"
#include "HAL/FileManager.h"
#include "HAL/IConsoleManager.h"
#include "ISourceControlModule.h"
#include "ISourceControlProvider.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "Misc/ScopedSlowTask.h"
#include "Misc/CoreDelegates.h"
#include "PackagesDialog.h"
#include "Serialization/JsonSerializer.h"
#include "Serialization/JsonWriter.h"
#include "SourceControlHelpers.h"
#include "SourceControlOperations.h"
#include "SourceControlPreferences.h"
#include "Styling/SlateTypes.h"
#include "UObject/ObjectSaveContext.h"
#include "Engine/AssetManager.h"

#define LOCTEXT_NAMESPACE "UncontrolledChangelists"

class FUncontrolledChangelistsDiscoverAssetsTask : public FNonAbandonableTask
{
public:
	FUncontrolledChangelistsDiscoverAssetsTask(FUncontrolledChangelistsModule* InOwner, FARFilter InAssetFilter)
		: Owner(InOwner)
		, AssetFilter(MoveTemp(InAssetFilter))
	{}
	~FUncontrolledChangelistsDiscoverAssetsTask() {}

	TStatId GetStatId() const
	{
		RETURN_QUICK_DECLARE_CYCLE_STAT(FUncontrolledChangelistsDiscoverAssetsTask, STATGROUP_ThreadPoolAsyncTasks);
	}

	void DoWork();
	const TSet<FString>& GetAddedAssetsCache() const { return AddedAssetsCache; }

private:
	FUncontrolledChangelistsModule* Owner;
	FARFilter AssetFilter;
	TSet<FString> AddedAssetsCache;
};

void FUncontrolledChangelistsDiscoverAssetsTask::DoWork()
{
	double StartTime = FPlatformTime::Seconds();
	UE_LOGF(LogSourceControl, Log, "Uncontrolled asset discovery started...");

	IAssetRegistry& AssetRegistry = IAssetRegistry::GetChecked();

	TArray<FAssetData> Assets;
	if (AssetFilter.IsEmpty())
	{
		const bool bIncludeOnlyOnDiskAssets = true;
		AssetRegistry.GetAllAssets(Assets, bIncludeOnlyOnDiskAssets);
	}
	else
	{
		AssetFilter.bIncludeOnlyOnDiskAssets = true;
		AssetRegistry.GetAssets(AssetFilter, Assets);
	}

	for (const FAssetData& AssetData : Assets) 
	{ 
		if (Owner->IsStopAssetDiscoveryRequested())
		{
			break;
		}
		Owner->OnAssetAddedInternal(AssetData, AddedAssetsCache, true);
	}
	
	UE_LOGF(LogSourceControl, Log, "Uncontrolled asset discovery finished in %ls seconds (Found %d uncontrolled assets)", *FString::SanitizeFloat(FPlatformTime::Seconds() - StartTime), AddedAssetsCache.Num());;
}

FUncontrolledChangelistsModule::FUncontrolledChangelistsModule() = default;
FUncontrolledChangelistsModule::~FUncontrolledChangelistsModule() = default;

void FUncontrolledChangelistsModule::StartupModule()
{
	bIsEnabled = USourceControlPreferences::AreUncontrolledChangelistsEnabled();

	const bool bIsEnabledThisFrame = IsEnabled();
	bWasEnabledLastFrame = bIsEnabledThisFrame;
	
	if (bIsEnabledThisFrame)
	{
		OnEnabled();
	}

	OnEndFrameDelegateHandle = FCoreDelegates::OnEndFrame.AddRaw(this, &FUncontrolledChangelistsModule::OnEndFrame);
}

void FUncontrolledChangelistsModule::ShutdownModule()
{
	checkf(!DiscoverAssetsTask, TEXT("The discover assets task should be cleaned up when OnEnginePreExit is called at the latest"));

	FCoreDelegates::OnEndFrame.Remove(OnEndFrameDelegateHandle);
	OnEndFrameDelegateHandle.Reset();

	OnDisabled();
}

void FUncontrolledChangelistsModule::OnEnabled()
{
	// Adds Default Uncontrolled Changelist if it is not already present.
	GetDefaultUncontrolledChangelistState();

	LoadState();

	OnObjectPreSavedDelegateHandle = FCoreUObjectDelegates::OnObjectPreSave.AddRaw(this, &FUncontrolledChangelistsModule::OnObjectPreSaved);
	OnCustomProjectsChangedDelegateHandle = ISourceControlModule::Get().OnCustomProjectsChanged().AddRaw(this, &FUncontrolledChangelistsModule::RequestReloadState);

	// Create initial scan event object
	InitialScanEvent = MakeShared<FInitialScanEvent>();

	UAssetManager::CallOrRegister_OnCompletedInitialScan(FSimpleMulticastDelegate::FDelegate::CreateLambda([this, WeakScanEvent = InitialScanEvent->AsWeak()]()
	{
		// Weak here allows us to check if module as been shutdown before using [this]
		if(!WeakScanEvent.IsValid())
		{
			return;
		}
		InitialScanEvent.Reset();

		StartAssetDiscovery();
		OnEnginePreExitDelegateHandle = FCoreDelegates::OnEnginePreExit.AddRaw(this, &FUncontrolledChangelistsModule::StopAssetDiscovery);

		IAssetRegistry& AssetRegistry = IAssetRegistry::GetChecked();
		OnAssetAddedDelegateHandle = AssetRegistry.OnAssetAdded().AddRaw(this, &FUncontrolledChangelistsModule::OnAssetAdded);
	}));
}

void FUncontrolledChangelistsModule::OnDisabled()
{
	// This will make sure callback for initial scan early outs if feature was disabled
	InitialScanEvent.Reset();
	
	StopAssetDiscovery();

	if (bIsStateDirty)
	{
		SaveState();
		check(!bIsStateDirty); // Should be cleared SaveState
	}

	// Check in case AssetRegistry has already been shutdown.
	if (IAssetRegistry* AssetRegistry = IAssetRegistry::Get())
	{
		AssetRegistry->OnAssetAdded().Remove(OnAssetAddedDelegateHandle);
	}
	OnAssetAddedDelegateHandle.Reset();

	FCoreUObjectDelegates::OnObjectPreSave.Remove(OnObjectPreSavedDelegateHandle);
	OnObjectPreSavedDelegateHandle.Reset();

	if (ISourceControlModule* SourceControl = ISourceControlModule::GetPtr())
	{
		SourceControl->OnCustomProjectsChanged().Remove(OnCustomProjectsChangedDelegateHandle);
	}
	OnCustomProjectsChangedDelegateHandle.Reset();

	FCoreDelegates::OnEnginePreExit.Remove(OnEnginePreExitDelegateHandle);
	OnEnginePreExitDelegateHandle.Reset();

	bPendingReloadState = false;
	LoadedCustomProjects.Reset();
	AddedAssetsCache.Reset();
	UncontrolledChangelistsStateCache.Reset();
}

bool FUncontrolledChangelistsModule::IsEnabled() const
{
	return bIsEnabled && ISourceControlModule::Get().GetProvider().UsesUncontrolledChangelists();
}

TArray<FUncontrolledChangelistStateRef> FUncontrolledChangelistsModule::GetChangelistStates() const
{
	TArray<FUncontrolledChangelistStateRef> UncontrolledChangelistStates;

	if (IsEnabled())
	{
		Algo::Transform(UncontrolledChangelistsStateCache, UncontrolledChangelistStates, [](const auto& Pair) { return Pair.Value; });
	}

	return UncontrolledChangelistStates;
}

FUncontrolledChangelistStatePtr FUncontrolledChangelistsModule::GetChangelistState(const FUncontrolledChangelist& InUncontrolledChangelist) const
{
	if (!IsEnabled())
	{
		return nullptr;
	}

	if (const FUncontrolledChangelistStateRef* UncontrolledChangelistState = UncontrolledChangelistsStateCache.Find(InUncontrolledChangelist))
	{
		return *UncontrolledChangelistState;
	}

	return nullptr;
}

FUncontrolledChangelistStatePtr FUncontrolledChangelistsModule::GetDefaultChangelistState() const
{
	return GetChangelistState(FUncontrolledChangelist(FUncontrolledChangelist::DEFAULT_UNCONTROLLED_CHANGELIST_GUID));
}

void FUncontrolledChangelistsModule::HandleChangelistStateModified()
{
	OnStateChanged();
}

bool FUncontrolledChangelistsModule::OnMakeWritable(const FString& InFilename)
{
	if (!IsEnabled())
	{
		return false;
	}

	AddedAssetsCache.Add(FPaths::ConvertRelativePathToFull(InFilename));
	return true;
}

bool FUncontrolledChangelistsModule::OnNewFilesAdded(const TArray<FString>& InFilenames)
{
	return AddToUncontrolledChangelist(InFilenames);
}

bool FUncontrolledChangelistsModule::OnSaveWritable(const FString& InFilename)
{
	return AddToUncontrolledChangelist({ InFilename });
}

bool FUncontrolledChangelistsModule::OnDeleteWritable(const FString& InFilename)
{
	return AddToUncontrolledChangelist({ InFilename });
}

bool FUncontrolledChangelistsModule::AddToUncontrolledChangelist(const TArray<FString>& InFilenames)
{
	if (!IsEnabled())
	{
		return false;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(FUncontrolledChangelistsModule::AddToUncontrolledChangelist);

	TArray<FString> FullPaths;
	FullPaths.Reserve(InFilenames.Num());
	Algo::Transform(InFilenames, FullPaths, [](const FString& Filename) { return FPaths::ConvertRelativePathToFull(Filename); });

	if (HasCustomProjectFilter())
	{
		FullPaths.RemoveAll([this](const FString& Filename)
		{
			return !DoesFilePassCustomProjectFilter(Filename);
		});
	}

	// Remove from reconcile cache
	for (const FString& FullPath : FullPaths)
	{
		AddedAssetsCache.Remove(FullPath);
	}

	// Group files by their UncontrolledChangelist and then add each group to it's corresponding UncontrolledChangelist.
	TMap<FUncontrolledChangelist, TArray<FString>> UncontrolledChangelistToFullPaths;
	GroupFilesByUncontrolledChangelist(MoveTemp(FullPaths), UncontrolledChangelistToFullPaths);

	bool bAreAllFilesAdded = true;

	for (const TTuple<FUncontrolledChangelist, TArray<FString>>& Pair : UncontrolledChangelistToFullPaths)
	{
		const FUncontrolledChangelist& UncontrolledChangelist = Pair.Key;
		const TArray<FString>& UncontrolledChangelistFilenames = Pair.Value;

		bAreAllFilesAdded &= AddFilesToUncontrolledChangelist(UncontrolledChangelist, UncontrolledChangelistFilenames, FUncontrolledChangelistState::ECheckFlags::NotCheckedOut);
	}

	return bAreAllFilesAdded;
}

bool FUncontrolledChangelistsModule::RemoveFromUncontrolledChangelist(const TArray<FString>& InFilenames)
{
	if (!IsEnabled())
	{
		return false;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(FUncontrolledChangelistsModule::RemoveFromUncontrolledChangelist);

	TArray<FString> FullPaths;
	FullPaths.Reserve(InFilenames.Num());
	Algo::Transform(InFilenames, FullPaths, [](const FString& Filename) { return FPaths::ConvertRelativePathToFull(Filename); });

	if (HasCustomProjectFilter())
	{
		FullPaths.RemoveAll([this](const FString& Filename)
		{
			return !DoesFilePassCustomProjectFilter(Filename);
		});
	}

	// Remove from reconcile cache
	for (const FString& FullPath : FullPaths)
	{
		AddedAssetsCache.Remove(FullPath);
	}

	// Group files by their UncontrolledChangelist and then remove each group from it's corresponding UncontrolledChangelist.
	TMap<FUncontrolledChangelist, TArray<FString>> UncontrolledChangelistToFullPaths;
	GroupFilesByUncontrolledChangelist(MoveTemp(FullPaths), UncontrolledChangelistToFullPaths);

	bool bAreAnyFilesRemoved = false;

	for (const TTuple<FUncontrolledChangelist, TArray<FString>>& Pair : UncontrolledChangelistToFullPaths)
	{
		const FUncontrolledChangelist& UncontrolledChangelist = Pair.Key;
		const TArray<FString>& UncontrolledChangelistFilenames = Pair.Value;

		bAreAnyFilesRemoved |= RemoveFilesFromUncontrolledChangelist(UncontrolledChangelist, UncontrolledChangelistFilenames);
	}

	return bAreAnyFilesRemoved;
}

void FUncontrolledChangelistsModule::GroupFilesByUncontrolledChangelist(TArray<FString>&& InFilenames, TMap<FUncontrolledChangelist, TArray<FString>>& OutUncontrolledChangelistToFilenames) const
{
	for (const TTuple<FUncontrolledChangelist, TSharedRef<FUncontrolledChangelistState>>& Pair : UncontrolledChangelistsStateCache)
	{
		const FUncontrolledChangelist& UncontrolledChangelist = Pair.Key;
		const FUncontrolledChangelistStateRef& UncontrolledChangelistState = Pair.Value;

		TArray<FString>& UncontrolledChangelistFilenames = OutUncontrolledChangelistToFilenames.FindOrAdd(UncontrolledChangelist);

		for (const FString& Filename : InFilenames)
		{
			if (UncontrolledChangelistState->ContainsFilename(Filename))
			{
				UncontrolledChangelistFilenames.Add(Filename);
			}
		}

		for (const FString& Filename : UncontrolledChangelistFilenames)
		{
			InFilenames.Remove(Filename);
		}
	}

	TArray<FString>& DefaultUncontrolledChangelistFilenames = OutUncontrolledChangelistToFilenames.FindOrAdd(FUncontrolledChangelist{ FUncontrolledChangelist::DEFAULT_UNCONTROLLED_CHANGELIST_GUID });
	for (const FString& Filename : InFilenames)
	{
		DefaultUncontrolledChangelistFilenames.Add(Filename);
	}
}

void FUncontrolledChangelistsModule::UpdateStatus()
{
	bool bHasStateChanged = false;

	if (!IsEnabled())
	{
		return;
	}

	for (FUncontrolledChangelistsStateCache::ElementType& Pair : UncontrolledChangelistsStateCache)
	{
		FUncontrolledChangelistsStateCache::ValueType& UncontrolledChangelistState = Pair.Value;
		bHasStateChanged |= UncontrolledChangelistState->UpdateStatus();
	}

	if (bHasStateChanged)
	{
		OnStateChanged();
	}
}

FText FUncontrolledChangelistsModule::GetReconcileStatus() const
{
	if (InitialScanEvent.IsValid())
	{
		return LOCTEXT("WaitForAssetRegistryStatus", "Waiting for Asset Registry initial scan...");
	}

	if (DiscoverAssetsTask && !DiscoverAssetsTask->IsDone())
	{
		return LOCTEXT("ProcessingAssetsStatus", "Processing assets...");
	}

	return FText::Format(LOCTEXT("ReconcileStatus", "Assets to check for reconcile: {0}"), FText::AsNumber(AddedAssetsCache.Num()));
}

bool FUncontrolledChangelistsModule::OnReconcileAssets()
{
	FScopedSlowTask Scope(0, LOCTEXT("ProcessingAssetsProgress", "Processing assets"));
	const bool bShowCancelButton = false;
	const bool bAllowInPIE = false;
	Scope.MakeDialogDelayed(1.0f, bShowCancelButton, bAllowInPIE);

	if (DiscoverAssetsTask)
	{
		while (!DiscoverAssetsTask->WaitCompletionWithTimeout(0.016f))
		{
			Scope.EnterProgressFrame(0.f);
		}

		AddedAssetsCache.Append(DiscoverAssetsTask->GetTask().GetAddedAssetsCache());
		DiscoverAssetsTask.Reset();
	}

	if ((!IsEnabled()) || AddedAssetsCache.IsEmpty())
	{
		return false;
	}

	Scope.EnterProgressFrame(0.f, LOCTEXT("ReconcileAssetsProgress", "Reconciling assets"));

	CleanAssetsCaches();

	bool bHasStateChanged = AddFilesToUncontrolledChangelist(
		FUncontrolledChangelist{ FUncontrolledChangelist::DEFAULT_UNCONTROLLED_CHANGELIST_GUID },
		AddedAssetsCache.Array(), FUncontrolledChangelistState::ECheckFlags::All);

	AddedAssetsCache.Empty();

	return bHasStateChanged;
}

void FUncontrolledChangelistsModule::OnAssetAdded(const FAssetData& AssetData)
{
	if (!IsEnabled())
	{
		return;
	}

	OnAssetAddedInternal(AssetData, AddedAssetsCache, false);
}

void FUncontrolledChangelistsModule::OnAssetAddedInternal(const FAssetData& AssetData, TSet<FString>& InAddedAssetsCache, bool bInDiscoveryTask)
{
	if (AssetData.HasAnyPackageFlags(PKG_Cooked))
	{
		return;
	}

	FPackagePath PackagePath;
	if (!FPackagePath::TryFromPackageName(AssetData.PackageName, PackagePath))
	{
		return;
	}

	// No need to check for existence when running discovery task
	if (!bInDiscoveryTask)
	{
		if (FPackageName::IsTempPackage(PackagePath.GetPackageName()))
		{
			return; // Ignore temp packages
		}

		if(!FPackageName::DoesPackageExist(PackagePath, &PackagePath))
		{
			return; // If the package does not exist on disk there is nothing more to do
		}
	}

	const FString LocalFullPath(PackagePath.GetLocalFullPath());

	if (LocalFullPath.IsEmpty())
	{
		return;
	}

	FString Fullpath = FPaths::ConvertRelativePathToFull(LocalFullPath);

	if (Fullpath.IsEmpty())
	{
		return;
	}

	// No need for path check when running discovery task, as it's handled by the ARFilter used by the task
	if (!bInDiscoveryTask && !DoesFilePassCustomProjectFilter(Fullpath))
	{
		return;
	}

	if (ISourceControlModule::Get().GetProvider().UsesLocalReadOnlyState() && !IFileManager::Get().IsReadOnly(*Fullpath))
	{
		InAddedAssetsCache.Add(MoveTemp(Fullpath));
	}
}

static bool ExecuteRevertOperation(const TArray<FString>& InFilenames)
{
	ISourceControlModule& SourceControlModule = ISourceControlModule::Get();
	ISourceControlProvider& SourceControlProvider = SourceControlModule.GetProvider();
	TArray<FSourceControlStateRef> UpdatedFilestates;

	auto BuildFileString = [](const TArray<FString>& Files) -> FString
	{
		TStringBuilder<2048> Builder;
		Builder.Join(Files, TEXT(", "));
		return Builder.ToString();
	};

	if (SourceControlProvider.GetState(InFilenames, UpdatedFilestates, EStateCacheUsage::ForceUpdate) != ECommandResult::Succeeded)
	{
		UE_LOGF(LogSourceControl, Error, "Failed to update the revision control files states for %ls.", *BuildFileString(InFilenames));
		return false;
	}

	TArray<FString> FilesToDelete;
	TArray<FString> FilesToRevert;

	for (const FSourceControlStateRef& Filestate : UpdatedFilestates)
	{
		if (Filestate->IsSourceControlled())
		{
			FilesToRevert.Add(Filestate->GetFilename());
		}
		else
		{
			FilesToDelete.Add(Filestate->GetFilename());
		}
	}

	if (!FilesToRevert.IsEmpty())
	{
		TSharedRef<FSync> ForceSyncOperation = ISourceControlOperation::Create<FSync>();
		ForceSyncOperation->SetForce(true);
		ForceSyncOperation->SetLastSyncedFlag(true);

		if (SourceControlProvider.Execute(ForceSyncOperation, FilesToRevert) != ECommandResult::Succeeded)
		{
			UE_LOGF(LogSourceControl, Error, "Failed to sync the following files to a previous version: %ls.", *BuildFileString(FilesToRevert));
			return false;
		}
	}

	IFileManager& FileManager = IFileManager::Get();
	bool bSuccess = true;

	for (const FString& FileToDelete : FilesToDelete)
	{
		const bool bRequireExists = true;
		const bool bEvenReadOnly = false;
		const bool bQuiet = false;

		if (!FileManager.Delete(*FileToDelete, bRequireExists, bEvenReadOnly, bQuiet))
		{
			UE_LOGF(LogSourceControl, Error, "Failed to delete %ls.", *FileToDelete);
			bSuccess = false;
		}
	}

	SourceControlModule.GetOnFilesDeleted().Broadcast(FilesToDelete);

	return bSuccess;
}

bool FUncontrolledChangelistsModule::OnRevert(const TArray<FString>& InFilenames)
{
	bool bSuccess = false;

	if (!IsEnabled() || InFilenames.IsEmpty())
	{
		return true;
	}

	bSuccess = SourceControlHelpers::ApplyOperationAndReloadPackages(InFilenames, ExecuteRevertOperation);

	UpdateStatus();

	return bSuccess;
}

void FUncontrolledChangelistsModule::OnObjectPreSaved(UObject* InObject, FObjectPreSaveContext InPreSaveContext)
{
	if (!IsEnabled())
	{
		return;
	}

	// Make sure we are catching the top level asset object to avoid processing same package multiple times
	if (!InObject || !InObject->IsAsset())
	{
		return;
	}

	// Ignore procedural save and autosaves
	if (InPreSaveContext.IsProceduralSave() || InPreSaveContext.IsFromAutoSave())
	{
		return;
	}

	FString Fullpath = FPaths::ConvertRelativePathToFull(InPreSaveContext.GetTargetFilename());

	if (Fullpath.IsEmpty())
	{
		return;
	}

	AddedAssetsCache.Add(MoveTemp(Fullpath));
}

void FUncontrolledChangelistsModule::MoveFilesToUncontrolledChangelist(const TArray<FSourceControlStateRef>& InControlledFileStates, const TArray<FSourceControlStateRef>& InUncontrolledFileStates, const FUncontrolledChangelist& InUncontrolledChangelist)
{
	bool bHasStateChanged = false;

	if (!IsEnabled())
	{
		return;
	}

	FUncontrolledChangelistsStateCache::ValueType* ChangelistState = UncontrolledChangelistsStateCache.Find(InUncontrolledChangelist);

	if (ChangelistState == nullptr)
	{
		return;
	}

	TArray<FString> Filenames;
	if (InControlledFileStates.Num() > 0)
	{
		Algo::Transform(InControlledFileStates, Filenames, [](const FSourceControlStateRef& State) { return State->GetFilename(); });

		ISourceControlProvider& SourceControlProvider = ISourceControlModule::Get().GetProvider();
		TSharedRef<FRevert, ESPMode::ThreadSafe> RevertOperation = ISourceControlOperation::Create<FRevert>();

		// Revert controlled files
		RevertOperation->SetSoftRevert(true);
		SourceControlProvider.Execute(RevertOperation, Filenames);
	}

	// Removes selected Uncontrolled Files from their Uncontrolled Changelists
	for (const auto& Pair : UncontrolledChangelistsStateCache)
	{
		const FUncontrolledChangelistStateRef& UncontrolledChangelistState = Pair.Value;
		UncontrolledChangelistState->RemoveFiles(InUncontrolledFileStates);
	}

	Algo::Transform(InUncontrolledFileStates, Filenames, [](const FSourceControlStateRef& State) { return State->GetFilename(); });

	// Add all files to their UncontrolledChangelist
	bHasStateChanged = (*ChangelistState)->AddFiles(Filenames, FUncontrolledChangelistState::ECheckFlags::None);

	if (bHasStateChanged)
	{
		OnStateChanged();
	}
}

void FUncontrolledChangelistsModule::MoveFilesToUncontrolledChangelist(const TArray<FString>& InControlledFiles, const FUncontrolledChangelist& InUncontrolledChangelist)
{
	bool bHasStateChanged = false;

	if (!IsEnabled())
	{
		return;
	}

	FUncontrolledChangelistsStateCache::ValueType* ChangelistState = UncontrolledChangelistsStateCache.Find(InUncontrolledChangelist);

	if (ChangelistState == nullptr)
	{
		return;
	}

	const TArray<FString>& Filenames = InControlledFiles;

	ISourceControlProvider& SourceControlProvider = ISourceControlModule::Get().GetProvider();
	TSharedRef<FRevert, ESPMode::ThreadSafe> RevertOperation = ISourceControlOperation::Create<FRevert>();

	// Revert controlled files
	RevertOperation->SetSoftRevert(true);
	SourceControlProvider.Execute(RevertOperation, Filenames);

	// Remove files from any existing UncontrolledChangelist
	bHasStateChanged = RemoveFromUncontrolledChangelist(Filenames);

	// Add all files to their UncontrolledChangelist
	bHasStateChanged |= (*ChangelistState)->AddFiles(Filenames, FUncontrolledChangelistState::ECheckFlags::None);

	if (bHasStateChanged)
	{
		OnStateChanged();
	}
}

void FUncontrolledChangelistsModule::MoveFilesToControlledChangelist(const TArray<FSourceControlStateRef>& InUncontrolledFileStates, const FSourceControlChangelistPtr& InChangelist, TFunctionRef<bool(const TArray<FSourceControlStateRef>&)> InOpenConflictDialog)
{
	if (!IsEnabled())
	{
		return;
	}

	TArray<FString> UncontrolledFilenames;
	
	Algo::Transform(InUncontrolledFileStates, UncontrolledFilenames, [](const FSourceControlStateRef& State) { return State->GetFilename(); });
	MoveFilesToControlledChangelist(UncontrolledFilenames, InChangelist, InOpenConflictDialog);
}

void FUncontrolledChangelistsModule::MoveFilesToControlledChangelist(const TArray<FString>& InUncontrolledFiles, const FSourceControlChangelistPtr& InChangelist, TFunctionRef<bool(const TArray<FSourceControlStateRef>&)> InOpenConflictDialog)
{
	if (!IsEnabled())
	{
		return;
	}

	ISourceControlProvider& SourceControlProvider = ISourceControlModule::Get().GetProvider();
	TArray<FSourceControlStateRef> UpdatedFilestates;

	// Get updated filestates to check Checkout capabilities.
	SourceControlProvider.GetState(InUncontrolledFiles, UpdatedFilestates, EStateCacheUsage::ForceUpdate);

	TArray<FSourceControlStateRef> FilesConflicts;
	TArray<FString> FilesToAdd;
	TArray<FString> FilesToCheckout;
	TArray<FString> FilesToDelete;

	// Check if we can Checkout files or mark for add
	for (const FSourceControlStateRef& Filestate : UpdatedFilestates)
	{
		const FString& Filename = Filestate->GetFilename();

		if (!Filestate->IsSourceControlled())
		{
			FilesToAdd.Add(Filename);
		}
		else if (!IFileManager::Get().FileExists(*Filename))
		{
			FilesToDelete.Add(Filename);
		}
		else if (Filestate->CanCheckout())
		{
			FilesToCheckout.Add(Filename);
		}
		else
		{
			FilesConflicts.Add(Filestate);
			FilesToCheckout.Add(Filename);
		}
	}

	bool bCanProceed = true;

	// If we detected conflict, asking user if we should proceed.
	if (!FilesConflicts.IsEmpty())
	{
		bCanProceed = InOpenConflictDialog(FilesConflicts);
	}

	if (bCanProceed)
	{
		if (!FilesToCheckout.IsEmpty())
		{
			SourceControlProvider.Execute(ISourceControlOperation::Create<FCheckOut>(), InChangelist, FilesToCheckout);
		}

		if (!FilesToAdd.IsEmpty())
		{
			SourceControlProvider.Execute(ISourceControlOperation::Create<FMarkForAdd>(), InChangelist, FilesToAdd);
		}

		if (!FilesToDelete.IsEmpty())
		{
			SourceControlProvider.Execute(ISourceControlOperation::Create<FDelete>(), InChangelist, FilesToDelete);
		}

		// UpdateStatus so UncontrolledChangelists can remove files from their cache if they were present before checkout.
		UpdateStatus();
	}
}

TOptional<FUncontrolledChangelist> FUncontrolledChangelistsModule::CreateUncontrolledChangelist(const FText& InDescription, const TOptional<FUncontrolledChangelist>& InUncontrolledChangelist)
{
	if (!IsEnabled())
	{
		return TOptional<FUncontrolledChangelist>();
	}

	if (InUncontrolledChangelist)
	{
		if (InUncontrolledChangelist->IsDefault())
		{
			UE_LOGF(LogSourceControl, Error, "Cannot use CreateUncontrolledChangelist with the Default Uncontrolled Changelist.");
			return TOptional<FUncontrolledChangelist>();
		}

		if (FUncontrolledChangelistStateRef* UncontrolledChangelistState = UncontrolledChangelistsStateCache.Find(*InUncontrolledChangelist))
		{
			(*UncontrolledChangelistState)->SetDescription(InDescription);
			return *InUncontrolledChangelist;
		}
	}
	
	// Default constructor will generate a new GUID.
	FUncontrolledChangelist NewUncontrolledChangelist = InUncontrolledChangelist ? *InUncontrolledChangelist : FUncontrolledChangelist();
	UncontrolledChangelistsStateCache.Emplace(NewUncontrolledChangelist, MakeShared<FUncontrolledChangelistState>(NewUncontrolledChangelist, InDescription));

	OnStateChanged();

	return NewUncontrolledChangelist;
}

void FUncontrolledChangelistsModule::EditUncontrolledChangelist(const FUncontrolledChangelist& InUncontrolledChangelist, const FText& InNewDescription)
{
	if (!IsEnabled())
	{
		return;
	}

	if (InUncontrolledChangelist.IsDefault())
	{
		UE_LOGF(LogSourceControl, Error, "Cannot edit Default Uncontrolled Changelist.");
		return;
	}

	FUncontrolledChangelistStateRef* UncontrolledChangelistState = UncontrolledChangelistsStateCache.Find(InUncontrolledChangelist);

	if (UncontrolledChangelistState == nullptr)
	{
		UE_LOGF(LogSourceControl, Error, "Cannot find Uncontrolled Changelist %ls in cache.", *InUncontrolledChangelist.ToString());
		return;
	}

	(*UncontrolledChangelistState)->SetDescription(InNewDescription);

	OnStateChanged();
}

void FUncontrolledChangelistsModule::DeleteUncontrolledChangelist(const FUncontrolledChangelist& InUncontrolledChangelist)
{
	if (!IsEnabled())
	{
		return;
	}

	if (InUncontrolledChangelist.IsDefault())
	{
		UE_LOGF(LogSourceControl, Error, "Cannot delete Default Uncontrolled Changelist.");
		return;
	}

	FUncontrolledChangelistStateRef* UncontrolledChangelistState = UncontrolledChangelistsStateCache.Find(InUncontrolledChangelist);

	if (UncontrolledChangelistState == nullptr)
	{
		UE_LOGF(LogSourceControl, Error, "Cannot find Uncontrolled Changelist %ls in cache.", *InUncontrolledChangelist.ToString());
		return;
	}

	if ((*UncontrolledChangelistState)->ContainsFiles())
	{
		UE_LOGF(LogSourceControl, Error, "Cannot delete Uncontrolled Changelist %ls while it contains files.", *InUncontrolledChangelist.ToString());
		return;
	}

	// Get Deleted Offline files and move them to the Default UCL so that we don't lose them
	GetDefaultUncontrolledChangelistState()->AddFiles((*UncontrolledChangelistState)->GetDeletedOfflineFiles().Array(), FUncontrolledChangelistState::ECheckFlags::None);

	UncontrolledChangelistsStateCache.Remove(InUncontrolledChangelist);

	OnStateChanged();
}

void FUncontrolledChangelistsModule::OnStateChanged()
{
	bIsStateDirty = true;
}

void FUncontrolledChangelistsModule::OnEndFrame()
{
	bool bStateChanged = false;

	if (!IsEngineExitRequested())
	{
		if (const bool bIsEnabledThisFrame = IsEnabled();
			bIsEnabledThisFrame != bWasEnabledLastFrame)
		{
			if (bWasEnabledLastFrame)
			{
				OnDisabled();
			}
			else
			{
				OnEnabled();
			}
			bWasEnabledLastFrame = bIsEnabledThisFrame;
			bStateChanged = true;
		}

		if (bPendingReloadState)
		{
			ReloadState();
			check(!bPendingReloadState); // Should be cleared ReloadState
			bStateChanged = true;
		}
	}

	if (DiscoverAssetsTask && DiscoverAssetsTask->IsDone())
	{
		AddedAssetsCache.Append(DiscoverAssetsTask->GetTask().GetAddedAssetsCache());
		DiscoverAssetsTask.Reset();
	}

	if (bIsStateDirty)
	{
		bStateChanged = true;
		SaveState();
		check(!bIsStateDirty); // Should be cleared SaveState
	}

	if (bStateChanged)
	{
		OnUncontrolledChangelistModuleChanged.Broadcast();
	}
}

void FUncontrolledChangelistsModule::StartAssetDiscovery()
{
	checkf(!DiscoverAssetsTask, TEXT("StartAssetDiscovery while another task was still running! Call StopAssetDiscovery first!"));
	checkf(!InitialScanEvent, TEXT("StartAssetDiscovery called while the asset registry scan was still happening!"));

	FARFilter AssetFilter;
	if (HasCustomProjectFilter())
	{
		checkf(LoadedCustomProjects.Num() > 0, TEXT("HasCustomProjectFilter logic is incompatible with StartAssetDiscovery!"));

		FNameBuilder ProjectContentPackagePath;
		for (const FSourceControlProjectInfo& Project : LoadedCustomProjects)
		{
			for (const FString& ProjectContentDirectory : Project.ContentDirectories)
			{
				if (FPackageName::TryConvertFilenameToLongPackageName(ProjectContentDirectory, ProjectContentPackagePath))
				{
					AssetFilter.PackagePaths.Add(FName(ProjectContentPackagePath));
				}
			}
		}
		AssetFilter.bRecursivePaths = true;

		// If AssetFilter.PackagePaths is empty then it means the current set of custom projects haven't mounted their content yet, 
		// and so there would be nothing to find. We bail here as passing an empty filter would discover everything rather than nothing.
		if (AssetFilter.PackagePaths.IsEmpty())
		{
			UE_LOGF(LogSourceControl, Log, "Uncontrolled asset discovery was skipped due to having no custom project content paths");
			return;
		}
	}

	DiscoverAssetsTask = MakePimpl<FAsyncTask<FUncontrolledChangelistsDiscoverAssetsTask>>(this, MoveTemp(AssetFilter));
	DiscoverAssetsTask->StartBackgroundTask();
}

void FUncontrolledChangelistsModule::StopAssetDiscovery()
{
	if (DiscoverAssetsTask.IsValid())
	{
		if (DiscoverAssetsTask->Cancel())
		{
			UE_LOGF(LogSourceControl, Log, "Uncontrolled asset discovery was cancelled by request");
		}
		else if (!DiscoverAssetsTask->IsDone())
		{
			const double StartTime = FPlatformTime::Seconds();
			bStopAssetDiscoveryRequested = true;

			UE_LOGF(LogSourceControl, Log, "Waiting on uncontrolled asset discovery to stop...");
			DiscoverAssetsTask->EnsureCompletion();
			UE_LOGF(LogSourceControl, Log, "Uncontrolled asset discovery stopped after stalling for %.1f(s)", FPlatformTime::Seconds() - StartTime);

			bStopAssetDiscoveryRequested = false;
		}

		DiscoverAssetsTask.Reset();
	}
}

bool FUncontrolledChangelistsModule::IsStopAssetDiscoveryRequested() const
{
	return bStopAssetDiscoveryRequested;
}

void FUncontrolledChangelistsModule::CleanAssetsCaches()
{
	// Remove files we are already tracking in Uncontrolled Changelists
	for (FUncontrolledChangelistsStateCache::ElementType& Pair : UncontrolledChangelistsStateCache)
	{
		FUncontrolledChangelistsStateCache::ValueType& UncontrolledChangelistState = Pair.Value;
		UncontrolledChangelistState->RemoveDuplicates(AddedAssetsCache);
	}
}

bool FUncontrolledChangelistsModule::AddFilesToUncontrolledChangelist(const FUncontrolledChangelist& InUncontrolledChangelist, const TArray<FString>& InFilenames, const FUncontrolledChangelistState::ECheckFlags InCheckFlags)
{
	bool bHasStateChanged = false;

	if (FUncontrolledChangelistStatePtr UncontrolledChangelistState = GetUncontrolledChangelistState(InUncontrolledChangelist))
	{
		// Try to add files, they will be added only if they pass the required checks
		bHasStateChanged = UncontrolledChangelistState->AddFiles(InFilenames, InCheckFlags);
	}

	if (bHasStateChanged)
	{
		OnStateChanged();
	}

	return bHasStateChanged;
}

bool FUncontrolledChangelistsModule::RemoveFilesFromUncontrolledChangelist(const FUncontrolledChangelist& InUncontrolledChangelist, const TArray<FString>& InFilenames)
{
	bool bHasStateChanged = false;

	if (FUncontrolledChangelistStatePtr UncontrolledChangelistState = GetUncontrolledChangelistState(InUncontrolledChangelist))
	{
		bHasStateChanged = UncontrolledChangelistState->RemoveFiles(InFilenames);
	}

	if (bHasStateChanged)
	{
		OnStateChanged();
	}

	return bHasStateChanged;
}

FUncontrolledChangelistStateRef FUncontrolledChangelistsModule::GetDefaultUncontrolledChangelistState()
{
	return GetUncontrolledChangelistState(FUncontrolledChangelist{ FUncontrolledChangelist::DEFAULT_UNCONTROLLED_CHANGELIST_GUID }).ToSharedRef();
}

FUncontrolledChangelistStatePtr FUncontrolledChangelistsModule::GetUncontrolledChangelistState(const FUncontrolledChangelist& InUncontrolledChangelist)
{
	FUncontrolledChangelistStateRef* UncontrolledChangelistState = UncontrolledChangelistsStateCache.Find(InUncontrolledChangelist);

	if (UncontrolledChangelistState != nullptr)
	{
		return *UncontrolledChangelistState;
	}

	if (InUncontrolledChangelist.IsDefault())
	{
		return UncontrolledChangelistsStateCache.Emplace(InUncontrolledChangelist, MakeShared<FUncontrolledChangelistState>(InUncontrolledChangelist, FUncontrolledChangelistState::DEFAULT_UNCONTROLLED_CHANGELIST_DESCRIPTION));
	}

	return nullptr;
}

bool FUncontrolledChangelistsModule::HasCustomProjectFilter() const
{
	// Note: If these rules change then you'll also need to update the filtering logic in StartAssetDiscovery
	return LoadedCustomProjects.Num() > 0;
}

bool FUncontrolledChangelistsModule::DoesFilePassCustomProjectFilter(const FString& InFilename) const
{
	if (HasCustomProjectFilter())
	{
		for (const FSourceControlProjectInfo& Project : LoadedCustomProjects)
		{
			if (DoesFilePassCustomProjectFilter(InFilename, Project))
			{
				return true;
			}
		}
		return false;
	}

	return true;
}

bool FUncontrolledChangelistsModule::DoesFilePassCustomProjectFilter(const FString& InFilename, const FSourceControlProjectInfo& Project)
{
	return FPaths::IsUnderDirectory(InFilename, Project.ProjectDirectory);
}

void FUncontrolledChangelistsModule::SaveState()
{
	SanitizeState();

	auto SaveStateImpl = [this](const FString& PersistentFilePath, const TFunction<bool(const FString&)>& FilenameFilter)
	{
		TSharedPtr<FJsonObject> RootObject = MakeShareable(new FJsonObject);
		RootObject->SetNumberField(VERSION_NAME, VERSION_NUMBER);

		TArray<TSharedPtr<FJsonValue>> UncontrolledChangelistsArray;
		for (const auto& Pair : UncontrolledChangelistsStateCache)
		{
			const FUncontrolledChangelist& UncontrolledChangelist = Pair.Key;
			FUncontrolledChangelistStateRef UncontrolledChangelistState = Pair.Value;
			TSharedPtr<FJsonObject> UncontrolledChangelistObject = MakeShareable(new FJsonObject);

			UncontrolledChangelist.Serialize(UncontrolledChangelistObject.ToSharedRef());
			UncontrolledChangelistState->Serialize(UncontrolledChangelistObject.ToSharedRef(), FilenameFilter);

			UncontrolledChangelistsArray.Add(MakeShareable(new FJsonValueObject(UncontrolledChangelistObject)));
		}
		RootObject->SetArrayField(CHANGELISTS_NAME, UncontrolledChangelistsArray);

		using FStringWriter = TJsonWriter<TCHAR, TPrettyJsonPrintPolicy<TCHAR>>;
		using FStringWriterFactory = TJsonWriterFactory<TCHAR, TPrettyJsonPrintPolicy<TCHAR>>;

		FString RootObjectStr;
		TSharedRef<FStringWriter> Writer = FStringWriterFactory::Create(&RootObjectStr);
		FJsonSerializer::Serialize(RootObject.ToSharedRef(), Writer);

		if (FFileHelper::SaveStringToFile(RootObjectStr, *PersistentFilePath))
		{
			UE_LOGF(LogSourceControl, Display, "Uncontrolled Changelist persistency file saved %ls", *PersistentFilePath);
		}
	};

	if (LoadedCustomProjects.Num() > 0)
	{
		// One JSON file per-project
		for (const FSourceControlProjectInfo& Project : LoadedCustomProjects)
		{
			const FString ProjectName = FPaths::GetCleanFilename(Project.ProjectDirectory); // TODO: Add ProjectName to FSourceControlProjectInfo?
			SaveStateImpl(GetPersistentFilePath(ProjectName), [&Project](const FString& Filename)
			{
				return DoesFilePassCustomProjectFilter(Filename, Project);
			});
		}
	}
	else
	{
		// One JSON file for the whole UE project
		SaveStateImpl(GetPersistentFilePath(FString()), nullptr);
	}

	bIsStateDirty = false;
}

void FUncontrolledChangelistsModule::LoadState()
{
	auto LoadStateImpl = [this](const FString& PersistentFilePath)
	{
		FString ImportJsonString;
		if (!FFileHelper::LoadFileToString(ImportJsonString, *PersistentFilePath))
		{
			return;
		}

		TSharedRef<TJsonReader<>> JsonReader = TJsonReaderFactory<>::Create(ImportJsonString);

		TSharedPtr<FJsonObject> RootObject;
		if (!FJsonSerializer::Deserialize(JsonReader, RootObject))
		{
			UE_LOGF(LogSourceControl, Error, "Cannot deserialize RootObject.");
			return;
		}

		uint32 VersionNumber = 0;
		if (!RootObject->TryGetNumberField(VERSION_NAME, VersionNumber))
		{
			UE_LOGF(LogSourceControl, Error, "Cannot get field %ls.", VERSION_NAME);
			return;
		}

		if (VersionNumber > VERSION_NUMBER)
		{
			UE_LOGF(LogSourceControl, Error, "Version number is invalid (file: %u, current: %u).", VersionNumber, VERSION_NUMBER);
			return;
		}

		const TArray<TSharedPtr<FJsonValue>>* UncontrolledChangelistsArray = nullptr;
		if (!RootObject->TryGetArrayField(CHANGELISTS_NAME, UncontrolledChangelistsArray))
		{
			UE_LOGF(LogSourceControl, Error, "Cannot get field %ls.", CHANGELISTS_NAME);
			return;
		}

		for (const TSharedPtr<FJsonValue>& JsonValue : *UncontrolledChangelistsArray)
		{
			FUncontrolledChangelist TempKey;
			TSharedRef<FJsonObject> JsonObject = JsonValue->AsObject().ToSharedRef();

			if (!TempKey.Deserialize(JsonObject))
			{
				UE_LOGF(LogSourceControl, Error, "Cannot deserialize FUncontrolledChangelist.");
				continue;
			}

			FUncontrolledChangelistsStateCache::ValueType& UncontrolledChangelistState = UncontrolledChangelistsStateCache.FindOrAdd(MoveTemp(TempKey), MakeShared<FUncontrolledChangelistState>(TempKey));

			UncontrolledChangelistState->Deserialize(JsonObject);
		}

		UE_LOGF(LogSourceControl, Display, "Uncontrolled Changelist persistency file loaded %ls", *PersistentFilePath);
	};

	LoadedCustomProjects = ISourceControlModule::Get().GetCustomProjects();
	if (LoadedCustomProjects.Num() > 0)
	{
		// One JSON file per-project
		for (const FSourceControlProjectInfo& Project : LoadedCustomProjects)
		{
			const FString ProjectName = FPaths::GetCleanFilename(Project.ProjectDirectory); // TODO: Add ProjectName to FSourceControlProjectInfo?
			LoadStateImpl(GetPersistentFilePath(ProjectName));
		}
	}
	else
	{
		// One JSON file for the whole UE project
		LoadStateImpl(GetPersistentFilePath(FString()));
	}

	SanitizeState();
}

void FUncontrolledChangelistsModule::RequestReloadState()
{
	bPendingReloadState = true;
}

void FUncontrolledChangelistsModule::ReloadState()
{
	// If the list of projects hasn't actually changed then we can skip this reload
	if (TArray<FSourceControlProjectInfo> NewCustomProjects = ISourceControlModule::Get().GetCustomProjects();
		NewCustomProjects == LoadedCustomProjects)
	{
		bPendingReloadState = false;
		return;
	}

	if (bIsStateDirty)
	{
		SaveState();
		check(!bIsStateDirty); // Should be cleared SaveState
	}

	// Clear the assets pending reconcile, as we will rebuild that list against the new project roots
	StopAssetDiscovery();
	AddedAssetsCache.Reset();

	// Clear any current uncontrolled changelist state, as we will load that from the new project JSON files
	UncontrolledChangelistsStateCache.Reset();
	GetDefaultUncontrolledChangelistState();

	LoadState();

	if (!InitialScanEvent)
	{
		StartAssetDiscovery();
	}

	bPendingReloadState = false;
}

void FUncontrolledChangelistsModule::SanitizeState()
{
	TSet<FString> AllFiles;

	for (const TPair<FUncontrolledChangelist, FUncontrolledChangelistStateRef>& Pair : UncontrolledChangelistsStateCache)
	{
		FUncontrolledChangelistStateRef UncontrolledChangelistState = Pair.Value;

		// UncontrolledChangelistState->Files
		{
			for (TSet<FSourceControlStateRef>::TIterator FileStateIt = UncontrolledChangelistState->Files.CreateIterator(); FileStateIt; ++FileStateIt)
			{
				if (AllFiles.Contains((*FileStateIt)->GetFilename()))
				{
					FileStateIt.RemoveCurrent();
				}
				else
				{
					AllFiles.Add((*FileStateIt)->GetFilename());
				}
			}

		}

		auto RemoveDuplicateFiles = [&AllFiles](TSet<FString>& Files)
		{
			for (TSet<FString>::TIterator FileIt = Files.CreateIterator(); FileIt; ++FileIt)
			{
				if (AllFiles.Contains(*FileIt))
				{
					FileIt.RemoveCurrent();
				}
				else
				{
					AllFiles.Add(*FileIt);
				}
			}
		};

		RemoveDuplicateFiles(UncontrolledChangelistState->OfflineFiles);
		RemoveDuplicateFiles(UncontrolledChangelistState->DeletedOfflineFiles);
	}
}

FString FUncontrolledChangelistsModule::GetPersistentFilePath(const FString& SubProjectName) const
{
	const FString Filename = SubProjectName.IsEmpty() ? TEXT("UncontrolledChangelists.json") : FString::Printf(TEXT("UncontrolledChangelists_%s.json"), *SubProjectName);
	return FPaths::Combine(FPaths::ProjectSavedDir(), TEXT("SourceControl"), Filename);
}

IMPLEMENT_MODULE(FUncontrolledChangelistsModule, UncontrolledChangelists);

#undef LOCTEXT_NAMESPACE
