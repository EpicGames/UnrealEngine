// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "ISourceControlChangelist.h"
#include "Misc/Guid.h"
#include "Templates/SharedPointer.h"

#define UE_API UNCONTROLLEDCHANGELISTS_API

class FText;

class FUncontrolledChangelist : public TSharedFromThis<FUncontrolledChangelist, ESPMode::ThreadSafe>
{
public:
	static constexpr const TCHAR* GUID_NAME = TEXT("guid");
	static UE_API const FGuid DEFAULT_UNCONTROLLED_CHANGELIST_GUID;

public:
	UE_API FUncontrolledChangelist();
	
	UE_API FUncontrolledChangelist(const FGuid& InGuid);
	
	bool operator==(const FUncontrolledChangelist& InOther) const
	{
		return Guid == InOther.Guid;
	}

	bool operator!=(const FUncontrolledChangelist& InOther) const
	{
		return Guid != InOther.Guid;
	}

	bool IsDefault() const
	{
		return Guid == DEFAULT_UNCONTROLLED_CHANGELIST_GUID;
	}

	friend FORCEINLINE uint32 GetTypeHash(const FUncontrolledChangelist& PerforceUncontrolledChangelist)
	{
		return GetTypeHash(PerforceUncontrolledChangelist.Guid);
	}

	FString ToString() const
	{
		return Guid.ToString();
	}

	/**
	 * Serialize the Uncontrolled Changelist to a Json Object.
	 * @param	OutJsonObject	The Json Object used to serialize.
	 */
	UE_API void Serialize(TSharedRef<class FJsonObject> OutJsonObject) const;
	
	/**
	 * Deserialize the Uncontrolled Changelist from a Json Object.
	 * @param 	InJsonValue 	The Json Object to read from.
	 * @return 	True if Deserialization succeeded.
	 */
	UE_API bool Deserialize(const TSharedRef<class FJsonObject> InJsonValue);

private:
	FGuid		Guid;
};

typedef TSharedPtr<FUncontrolledChangelist, ESPMode::ThreadSafe> FUncontrolledChangelistPtr;
typedef TSharedRef<FUncontrolledChangelist, ESPMode::ThreadSafe> FUncontrolledChangelistRef;

#undef UE_API
