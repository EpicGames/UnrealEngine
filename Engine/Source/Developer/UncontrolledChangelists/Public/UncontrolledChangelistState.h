// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Set.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "ISourceControlChangelistState.h"
#include "ISourceControlOperation.h"
#include "ISourceControlProvider.h"
#include "ISourceControlState.h"
#include "Internationalization/Text.h"
#include "Misc/DateTime.h"
#include "Misc/EnumClassFlags.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"
#include "UncontrolledChangelist.h"

#define UE_API UNCONTROLLEDCHANGELISTS_API

class FJsonObject;

class FUncontrolledChangelistState : public TSharedFromThis<FUncontrolledChangelistState, ESPMode::ThreadSafe>
{
public:
	static constexpr const TCHAR* FILES_NAME = TEXT("files");
	static constexpr const TCHAR* NAME_NAME = TEXT("name");
	static constexpr const TCHAR* DESCRIPTION_NAME = TEXT("description");
	static UE_API const FText DEFAULT_UNCONTROLLED_CHANGELIST_DESCRIPTION;

	enum class ECheckFlags
	{
		/** No Check */
		None			= 0,

		/** File has been modified */
		Modified		= 1,

		/** File is not checked out */
		NotCheckedOut	= 1 << 1,

		/** All the above checks */
		All = Modified | NotCheckedOut,
	};

public:
	// An FUncontrolledChangelistState should always reference a given Changelist (with a valid GUID).
	FUncontrolledChangelistState() = delete;

	UE_API FUncontrolledChangelistState(const FUncontrolledChangelist& InUncontrolledChangelist);
	
	UE_API FUncontrolledChangelistState(const FUncontrolledChangelist& InUncontrolledChangelist, const FText& InDescription);

	// Uncontrolled Changelist states are unique and non-copyable, should always be used by reference to preserve cache coherence.
	FUncontrolledChangelistState(const FUncontrolledChangelistState& InUncontrolledChangelistState) = delete;
	FUncontrolledChangelistState& operator=(const FUncontrolledChangelistState& InUncontrolledChangelistState) = delete;

	/**
	 * Get the name of the icon graphic we should use to display the state in a UI.
	 * @returns the name of the icon to display
	 */
	 UE_API FName GetIconName() const;

	/**
	 * Get the name of the small icon graphic we should use to display the state in a UI.
	 * @returns the name of the icon to display
	 */
	UE_API FName GetSmallIconName() const;

	/**
	 * Get a text representation of the state
	 * @returns	the text to display for this state
	 */
	UE_API const FText& GetDisplayText() const;

	/**
	 * Get a text representation of the state
	 * @returns	the text to display for this state
	 */
	UE_API const FText& GetDescriptionText() const;

	/**
	 * Get a tooltip to describe this state
	 * @returns	the text to display for this states tooltip
	 */
	UE_API FText GetDisplayTooltip() const;

	/**
	 * Get the timestamp of the last update that was made to this state.
	 * @returns	the timestamp of the last update
	 */
	UE_API const FDateTime& GetTimeStamp() const;

	UE_API const TSet<FSourceControlStateRef>& GetFilesStates() const;
	
	UE_API const TSet<FString>& GetOfflineFiles() const;

	UE_API const TSet<FString>& GetDeletedOfflineFiles() const;

	/**
	 * Get the number of files in the CL. (Includes file states and offline files) 
	 */
	UE_API int32 GetFileCount() const;

	/**
	 * Get the filenames in the CL. (Includes file states and offline files)
	 */
	UE_API TArray<FString> GetFilenames() const;

	/**
	 * Check whether a file exists in the file states or offline files
	 */
	UE_API bool ContainsFilename(const FString& PackageFilename) const;

	/**
	 * Serialize the state of the Uncontrolled Changelist to a Json Object, optionally filtering the files.
	 * @param 	OutJsonObject 	The Json object used to serialize.
	 * @param	FilenameFilter	An optional filter to control while filenames are included in the serialized JSON.
	 */
	UE_API void Serialize(TSharedRef<class FJsonObject> OutJsonObject, const TFunction<bool(const FString&)>& FilenameFilter = nullptr) const;

	/**
	 * Deserialize the state of the Uncontrolled Changelist from a Json Object.
	 * @param 	InJsonValue 	The Json Object to read from.
	 * @return 	True if Deserialization succeeded.
	 */
	UE_API bool Deserialize(const TSharedRef<FJsonObject> InJsonValue);

	/**
	 * Adds files to this Uncontrolled Changelist State.
	 * @param 	InFilenames		The files to be added.
	 * @param 	InCheckFlags 	Tells which checks have to pass to add a file.
	 * @return 	True if a change has been performed in the Uncontrolled Changelist State.
	 */
	UE_API bool AddFiles(const TArray<FString>& InFilenames, const ECheckFlags InCheckFlags);

	/**
	 * Removes files from this Uncontrolled Changelist State if present.
	 * @param 	InFileStates 	The files to be removed.
	 * @return 	True if a change has been performed in the Uncontrolled Changelist State.
	 */
	UE_API bool RemoveFiles(const TArray<FSourceControlStateRef>& InFileStates);

	/**
	 * Removes the given filenames from the file states or offline files.
	 * @param 	InFilenames 	The filenames to be removed.
	 * @return 	True if a change has been performed in the Uncontrolled Changelist State.
	 */
	UE_API bool RemoveFiles(const TArray<FString>& InFilenames);

	/**
	 * Updates the status of all files contained in this changelist.
	 * @return 	True if the state has been modified.
	 */
	UE_API bool UpdateStatus();

	/**
	 * Removes files present both in the Uncontrolled Changelist and the provided set.
	 * @param 	InOutAddedAssets 	A Set representing Added Assets to check against.
	 */
	UE_API void RemoveDuplicates(TSet<FString>& InOutAddedAssets);

	/**
	 * Sets a new description for this Uncontrolled Changelist
	 * @param	InDescription	The new description to set.
	 */
	UE_API void SetDescription(const FText& InDescription);

	/** Returns true if the Uncontrolled Changelists contains either Files or OfflineFiles.	*/
	UE_API bool ContainsFiles() const;

public:
	FUncontrolledChangelist Changelist;
	FText Description;
	TSet<FSourceControlStateRef> Files;
	TSet<FString> OfflineFiles;
	TSet<FString> DeletedOfflineFiles;

	/** The timestamp of the last update */
	FDateTime TimeStamp;
};

ENUM_CLASS_FLAGS(FUncontrolledChangelistState::ECheckFlags);
typedef TSharedPtr<FUncontrolledChangelistState, ESPMode::ThreadSafe> FUncontrolledChangelistStatePtr;
typedef TSharedRef<FUncontrolledChangelistState, ESPMode::ThreadSafe> FUncontrolledChangelistStateRef;

#undef UE_API
