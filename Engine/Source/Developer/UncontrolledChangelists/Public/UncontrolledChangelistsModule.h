// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "UncontrolledChangelistState.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "UObject/UObjectGlobals.h"
#include "Async/AsyncWork.h"

#define UE_API UNCONTROLLEDCHANGELISTS_API

struct FAssetData;
struct FSourceControlProjectInfo;

class FObjectPreSaveContext;
class FUncontrolledChangelistsDiscoverAssetsTask;

/**
 * Interface for talking to Uncontrolled Changelists
 */
class FUncontrolledChangelistsModule : public IModuleInterface
{
	typedef TMap<FUncontrolledChangelist, FUncontrolledChangelistStateRef> FUncontrolledChangelistsStateCache;

public:	
	static constexpr const TCHAR* VERSION_NAME = TEXT("version");
	static constexpr const TCHAR* CHANGELISTS_NAME = TEXT("changelists");
	static constexpr uint32 VERSION_NUMBER = 1;

	/** Callback called when the state of the Uncontrolled Changelist Module (or any Uncontrolled Changelist) changed */
	DECLARE_MULTICAST_DELEGATE(FOnUncontrolledChangelistModuleChanged);
	FOnUncontrolledChangelistModuleChanged OnUncontrolledChangelistModuleChanged;

public:
	UE_API FUncontrolledChangelistsModule();
	UE_API ~FUncontrolledChangelistsModule();

	/** IModuleInterface implementation */
	UE_API virtual void StartupModule() override;
	UE_API virtual void ShutdownModule() override;

	/**
	 * Check whether uncontrolled changelist module is enabled.
	 */
	UE_API bool IsEnabled() const;

	/**
	 * Get the changelist state of each cached Uncontrolled Changelist.
	 */
	UE_API TArray<FUncontrolledChangelistStateRef> GetChangelistStates() const;

	/**
	 * Get the changelist state of the given Uncontrolled Changelist.
	 */
	UE_API FUncontrolledChangelistStatePtr GetChangelistState(const FUncontrolledChangelist& InUncontrolledChangelist) const;

	/**
	 * Get the changelist state of the default Uncontrolled Changelist.
	 */
	UE_API FUncontrolledChangelistStatePtr GetDefaultChangelistState() const;

	/**
	 * Called if the state of any Uncontrolled Changelist is modified externally, eg, via the mutable accessors above.
	 */
	UE_API void HandleChangelistStateModified();

	/**
	 * Called when file has been made writable. Adds the file to the reconcile list because we don't know yet if it will change.
	 * @param	InFilename		The file to be reconciled.
	 * @return True if the file have been handled by the Uncontrolled module.
	 */
	UE_API bool OnMakeWritable(const FString& InFilename);
	 	 
	/**
	 * Called when file has been saved without an available Provider. Adds the file to the Default Uncontrolled Changelist
	 * @param	InFilename		The file to be added.
	 * @return	True if the file have been handled by the Uncontrolled module.
	 */
	UE_API bool OnSaveWritable(const FString& InFilename);

	/**
	 * Called when file has been deleted without an available Provider. Adds the file to the Default Uncontrolled Changelist
	 * @param	InFilename		The file to be added.
	 * @return	True if the file have been handled by the Uncontrolled module.
	 */
	UE_API bool OnDeleteWritable(const FString& InFilename);

	/**
	 * Called when files should have been marked for add without an available Provider. Adds the files to the Default Uncontrolled Changelist
	 * @param	InFilenames		The files to be added.
	 * @return	True if the files have been handled by the Uncontrolled module.
	 */
	UE_API bool OnNewFilesAdded(const TArray<FString>& InFilenames);

	/**
	 * Updates the status of Uncontrolled Changelists and files.
	 */
	UE_API void UpdateStatus();

	/**
	 * Gets a reference to the UncontrolledChangelists module
	 * @return A reference to the UncontrolledChangelists module.
	 */
	static inline FUncontrolledChangelistsModule& Get()
	{
		return FModuleManager::LoadModuleChecked<FUncontrolledChangelistsModule>(GetModuleName());
	}

	/**
	 * Gets a pointer to the UncontrolledChangelists module, if loaded
	 * @return A pointer to the UncontrolledChangelists module.
	 */
	static inline FUncontrolledChangelistsModule* GetPtr()
	{
		return FModuleManager::GetModulePtr<FUncontrolledChangelistsModule>(GetModuleName());
	}

	/**
	 * Checks to see if this module is loaded and ready.  It is only valid to call Get() if IsAvailable() returns true.
	 *
	 * @return True if the module is loaded and ready to use
	 */
	static inline bool IsAvailable()
	{
		return FModuleManager::Get().IsModuleLoaded( GetModuleName() );
	}

	static FName GetModuleName()
	{
		static FName UncontrolledChangelistsModuleName("UncontrolledChangelists");
		return UncontrolledChangelistsModuleName;
	}

	/**
	 * Gets a message indicating the status of SCC coherence.
	 * @return 	A text representing the status of SCC.
	 */
	UE_API FText GetReconcileStatus() const;

	/** Called when "Reconcile assets" button is clicked. Checks for uncontrolled modifications in previously added assets.
	 *	Adds modified files to Uncontrolled Changelists
	 *  @return True if new modifications found
	 */
	UE_API bool OnReconcileAssets();

	/**
	 * Delegate callback called when assets are added to AssetRegistry.
	 * @param 	AssetData 	The asset just added.
	 */
	UE_API void OnAssetAdded(const FAssetData& AssetData);

	/** Called when "Revert files" button is clicked. Reverts modified files and deletes new ones.
	 *  @param	InFilenames		The files to be reverted
	 *	@return true if the provided files were reverted.
	 */
	UE_API bool OnRevert(const TArray<FString>& InFilenames);
	
	/**
	 * Delegate callback called before an asset has been written to disk.
	 * @param 	InObject 			The saved object.
	 * @param 	InPreSaveContext 	Interface used to access saved parameters.
	 */
	UE_API void OnObjectPreSaved(UObject* InObject, FObjectPreSaveContext InPreSaveContext);

	/**
	 * Moves files to an Uncontrolled Changelist.
	 * @param 	InControlledFileStates 		The Controlled files to move.
	 * @param 	InUncontrolledFileStates 	The Uncontrolled files to move.
	 * @param 	InUncontrolledChangelist 	The Uncontrolled Changelist where to move the files.
	 */
	UE_API void MoveFilesToUncontrolledChangelist(const TArray<FSourceControlStateRef>& InControlledFileStates, const TArray<FSourceControlStateRef>& InUncontrolledFileStates, const FUncontrolledChangelist& InUncontrolledChangelist);

	/**
	* Moves files to an Uncontrolled Changelist.
	* @param 	InControlledFileStates 		The Controlled files to move.
	* @param 	InUncontrolledChangelist 	The Uncontrolled Changelist where to move the files.
	*/
	UE_API void MoveFilesToUncontrolledChangelist(const TArray<FString>& InControlledFiles, const FUncontrolledChangelist& InUncontrolledChangelist);

	/**
	 * Moves files to a Controlled Changelist.
	 * @param 	InUncontrolledFileStates 	The files to move.
	 * @param 	InChangelist 				The Controlled Changelist where to move the files.
	 * @param 	InOpenConflictDialog 		A callback to be used by the method when file conflicts are detected. The callback should display the files and ask the user if they should proceed.
	 */
	UE_API void MoveFilesToControlledChangelist(const TArray<FSourceControlStateRef>& InUncontrolledFileStates, const FSourceControlChangelistPtr& InChangelist, TFunctionRef<bool(const TArray<FSourceControlStateRef>&)> InOpenConflictDialog);
	
	/**
	 * Moves files to a Controlled Changelist.
	 * @param 	InUncontrolledFiles 	The files to move.
	 * @param 	InChangelist 			The Controlled Changelist where to move the files.
	 * @param 	InOpenConflictDialog 	A callback to be used by the method when file conflicts are detected. The callback should display the files and ask the user if they should proceed.
	 */
	UE_API void MoveFilesToControlledChangelist(const TArray<FString>& InUncontrolledFiles, const FSourceControlChangelistPtr& InChangelist, TFunctionRef<bool(const TArray<FSourceControlStateRef>&)> InOpenConflictDialog);

	/**
	 * Creates a new Uncontrolled Changelist.
	 * @param	InDescription				The description of the newly created Uncontrolled Changelist.
	 * @param	InUncontrolledChangelist	An optional Uncontrolled Changelist to create (or find) via its ID. Should not be the default Uncontrolled Changelist.
	 * return	TOptional<FUncontrolledChangelist> set with the new Uncontrolled Changelist key if succeeded.
	 */
	UE_API TOptional<FUncontrolledChangelist> CreateUncontrolledChangelist(const FText& InDescription, const TOptional<FUncontrolledChangelist>& InUncontrolledChangelist = TOptional<FUncontrolledChangelist>());

	/**
     * Edits an Uncontrolled Changelist's description
	 * @param	InUncontrolledChangelist	The Uncontrolled Changelist to modify. Should not be the default Uncontrolled Changelist.
	 * @param	InNewDescription			The description to set.
	 */
	UE_API void EditUncontrolledChangelist(const FUncontrolledChangelist& InUncontrolledChangelist, const FText& InNewDescription);
	
	/**
	 * Deletes an Uncontrolled Changelist.
	 * @param	InUncontrolledChangelist	The Uncontrolled Changelist to delete. Should not be the default Uncontrolled Changelist and should not contain files.
	 */
	UE_API void DeleteUncontrolledChangelist(const FUncontrolledChangelist& InUncontrolledChangelist);

private:
	/**
	 * Helper use by asset discovery task and OnAssetLoaded delegate.
	 * @param 	InAssetData 		The asset just added.
	 * @param  	InAddedAssetsCache 	The cache to add the asset to.
	 * @param	bInDiscoveryTask	If true, this asset was added from the asset discovery task.
	 * 
	 */
	UE_API void OnAssetAddedInternal(const FAssetData& InAssetData, TSet<FString>& InAddedAssetsCache, bool bInDiscoveryTask);

	/**
	 * Add files to Uncontrolled Changelist. Also adds them to files to reconcile.
	 */
	UE_API bool AddToUncontrolledChangelist(const TArray<FString>& InFilenames);

	/**
	 * Removes the given files from their associated Uncontrolled Changelist, if any.  Also removes them from files to reconcile.
	 * @return true if any of the given files are removed.
	 */
	UE_API bool RemoveFromUncontrolledChangelist(const TArray<FString>& InFilenames);

	/**
	 * Groups the given files by their associated Uncontrolled Changelist and returns the groupings in the given map.
	 * Any of the files that are not already associated with an Uncontrolled Changelist will be added to the default Uncontrolled Changelist's group.
	 */
	UE_API void GroupFilesByUncontrolledChangelist(TArray<FString>&& InFilenames, TMap<FUncontrolledChangelist, TArray<FString>>& OutUncontrolledChangelistToFilenames) const;

	/**
	 * True if we have custom projects and calling DoesFilePassCustomProjectFilter could ever return something aside from true.
	 */
	UE_API bool HasCustomProjectFilter() const;

	/**
	 * Run the given file through the filter for all known custom projects.
	 */
	UE_API bool DoesFilePassCustomProjectFilter(const FString& InFilename) const;

	/**
	 * Run the given file through the filter for the given custom project.
	 */
	static UE_API bool DoesFilePassCustomProjectFilter(const FString& InFilename, const FSourceControlProjectInfo& Project);

	/**
	 * Saves the state of UncontrolledChangelists to Json for persistency.
	 */
	UE_API void SaveState();
	
	/**
	 * Restores the previously saved state from Json.
	 */
	UE_API void LoadState();
	
	/**
	 * Request that ReloadState be called at the end of the current frame.
	 */
	UE_API void RequestReloadState();

	/**
	 * Reload the state of the UncontrolledChangelists when ISourceControlModule::GetCustomProjects changes.
	 */
	UE_API void ReloadState();

	/**
	 * Removes any duplicated files across changelists
	 */
	UE_API void SanitizeState();

	/**
	 * Called on End of frame. Calls SaveState if needed.
	 */
	UE_API void OnEndFrame();

	/**
	 * Called when the uncontrolled changelist feature switches to an enabled state.
	 */
	UE_API void OnEnabled();

	/**
	 * Called when the uncontrolled changelist feature switches to a disabled state.
	 */
	UE_API void OnDisabled();

	/**
	 * Start a new DiscoverAssetsTask for the current state.
	 * @note InitialScanEvent must be null before calling this, and there shouldn't be any existing DiscoverAssetsTask running.
	 */
	UE_API void StartAssetDiscovery();

	/**
	 * Stop the current DiscoverAssetsTask, if any.
	 */
	UE_API void StopAssetDiscovery();

	/**
	 * Used by the task to query whether we're waiting for it to finish in StopAssetDiscovery. 
	 */
	UE_API bool IsStopAssetDiscoveryRequested() const;

	/**
	 * Helper returning the location of the file used for persistency.
	 * @return 	A string containing the filepath.
	 */
	UE_API FString GetPersistentFilePath(const FString& SubProjectName) const;

	/** Called when a state changed either in the module or an Uncontrolled Changelist. */
	UE_API void OnStateChanged();

	/** Removes from asset caches files already present in Uncontrolled Changelists */
	UE_API void CleanAssetsCaches();

	/**
	 * Try to add the provided filenames to the given Uncontrolled Changelist.
	 * @param	InUncontrolledChangelist	The Uncontrolled Changelist to add to.
	 * @param	InFilenames 				The files to add.
	 * @param	InCheckFlags 				The required checks to check the file against before adding.
	 * @return	True files have been added.
	 */
	UE_API bool AddFilesToUncontrolledChangelist(const FUncontrolledChangelist& InUncontrolledChangelist, const TArray<FString>& InFilenames, const FUncontrolledChangelistState::ECheckFlags InCheckFlags);

	/**
	 * Try to remove the provided filenames from the given Uncontrolled Changelist.
	 * @param	InUncontrolledChangelist	The Uncontrolled Changelist to remove from.
	 * @param	InFilenames 				The files to remove.
	 * @return	True files have been removed.
	 */
	UE_API bool RemoveFilesFromUncontrolledChangelist(const FUncontrolledChangelist& InUncontrolledChangelist, const TArray<FString>& InFilenames);

	/** Returns the default Uncontrolled Changelist state, creates it if it does not exist. */
	UE_API FUncontrolledChangelistStateRef GetDefaultUncontrolledChangelistState();

	/**
	 * Returns the given Uncontrolled Changelist state if it exists.
	 * If the given Uncontrolled Changelist is the default one and it does not exist, then it will be created and returned.
	 * Otherwise, if not the default Uncontrolled Changelist, then nullptr will be returned if it does not exist.
	 */
	UE_API FUncontrolledChangelistStatePtr GetUncontrolledChangelistState(const FUncontrolledChangelist& InUncontrolledChangelist);

private:
	friend FUncontrolledChangelistsDiscoverAssetsTask;

	// Used to determine if the initial Asset Registry scan was completed or the module was shutdown
	struct FInitialScanEvent : public TSharedFromThis<FInitialScanEvent> {};
	TSharedPtr<FInitialScanEvent> InitialScanEvent;

	TPimplPtr<FAsyncTask<FUncontrolledChangelistsDiscoverAssetsTask>> DiscoverAssetsTask;
	FUncontrolledChangelistsStateCache	UncontrolledChangelistsStateCache;
	TArray<FSourceControlProjectInfo>	LoadedCustomProjects;
	TSet<FString>						AddedAssetsCache;
	FDelegateHandle						OnEnginePreExitDelegateHandle;
	FDelegateHandle						OnAssetAddedDelegateHandle;
	FDelegateHandle						OnObjectPreSavedDelegateHandle;
	FDelegateHandle						OnCustomProjectsChangedDelegateHandle;
	FDelegateHandle						OnEndFrameDelegateHandle;
	std::atomic<bool>					bStopAssetDiscoveryRequested = false;
	bool								bIsEnabled = false;
	bool								bWasEnabledLastFrame = false;
	bool								bIsStateDirty = false;
	bool								bPendingReloadState = false;
};

#undef UE_API
