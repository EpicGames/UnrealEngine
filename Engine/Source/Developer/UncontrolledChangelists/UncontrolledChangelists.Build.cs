// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class UncontrolledChangelists : ModuleRules
{
	public UncontrolledChangelists(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"CoreUObject",
				"SlateCore",
				"InputCore",
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"Json",
				"SourceControl",
				"AssetRegistry",
			}
		);

		if (Target.bBuildEditor)
        {
			PrivateDependencyModuleNames.AddRange(
                new string[]
				{
					"EditorFramework",
					"Engine",
					"PackagesDialog",
				}
			);
        }
	}
}
